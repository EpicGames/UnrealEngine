;Format Entry=(Devices=<Major_Model_Name>:<Minor_Model_Name1>, <Minor_Model_Name2>, Resx=<ResX_Value>, ResY=<ResY_Value>)
[DeviceModelToDeviceSpec]
Entry=(Devices="iPhone10:3,6|iPhone11:2", ResX="2436" ResY="1125")
Entry=(Devices="iPhone11:4,6|iPhone12:5", ResX="2688", ResY="1242")
Entry=(Devices="iPhone11:8|iPhone12:1", ResX="1792", ResY="828")
Entry=(Devices="iPhone13:1|iPhone14:4", ResX="2340", ResY="1080")
Entry=(Devices="iPhone13:2,3|iPhone14:2,5,7", ResX="2532", ResY="1170")
Entry=(Devices="iPhone13:4|iPhone14:3,8", ResX="2778", ResY="1284")
Entry=(Devices="iPhone15:2,4|iPhone16:1", ResX="2556", ResY="1179")
Entry=(Devices="iPhone15:3,5|iPhone16:2|iPhone17:4", ResX="2796", ResY="1290")
Entry=(Devices="iPhone17:1,2|iPhone18:1,2", ResX="2622", ResY="1206")
Entry=(Devices="iPhone17:3|iPhone18:3", ResX="2868", ResY="1320")
Entry=(Devices="iPhone18:4", ResX="2736", ResY="1260")