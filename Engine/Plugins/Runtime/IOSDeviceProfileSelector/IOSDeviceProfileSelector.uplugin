{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "IOS Device Profile Selector",
	"Description" : "IOS Device Profile Selector used show selection of device profiles on hardware",
	"Category" : "Device Profile Selectors",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name": "IOSPreviewDeviceProfileSelector",
			"Type": "Editor",
			"LoadingPhase": "None"
		},
		{
			"Name" : "IOSDeviceProfileSelector",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" :
			[
				"IOS",
				"TVOS",
				"VisionOS"
			]
		}
	]
}
