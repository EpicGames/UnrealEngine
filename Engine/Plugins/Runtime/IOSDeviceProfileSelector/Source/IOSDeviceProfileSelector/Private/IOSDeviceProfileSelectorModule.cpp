// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSDeviceProfileSelectorModule.h"
#include "HAL/PlatformMisc.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FIOSDeviceProfileSelectorModule, IOSDeviceProfileSelector);


void FIOSDeviceProfileSelectorModule::StartupModule()
{
}


void FIOSDeviceProfileSelectorModule::ShutdownModule()
{
}


FString const FIOSDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	FString ProfileName = FPlatformMisc::GetDefaultDeviceProfileName();
	if( ProfileName.IsEmpty() )
	{
		ProfileName = FPlatformProperties::PlatformName();
	}
	UE_LOGF(LogIOS, Log, "Selected Device Profile: [%ls]", *ProfileName);

	// If no type was obtained, just select IOS as default
	return ProfileName;
}
