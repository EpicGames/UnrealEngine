// Copyright Epic Games, Inc. All Rights Reserved.
using System.IO;
using UnrealBuildBase;

namespace UnrealBuildTool.Rules
{
	public class IOSPreviewDeviceProfileSelector : ModuleRules
	{
		public IOSPreviewDeviceProfileSelector(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"RHI",
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Json",
					"JsonUtilities",
					"PIEPreviewDeviceSpecification",
					"Projects",
				}
				);


			if (IsPlatformAvailable(UnrealTargetPlatform.IOS) && (Target.Type == TargetType.Editor) && (Target.Platform == UnrealTargetPlatform.Win64 || (Target.Platform == UnrealTargetPlatform.Mac)))
			{
				PrivateDefinitions.Add("WITH_IOS_DEVICE_DETECTION=1");
				PublicIncludePathModuleNames.Add("IOSTargetPlatformControls");
				PrivateDependencyModuleNames.Add("IOSTargetPlatformControls");
			}
			else 
			{
				PrivateDefinitions.Add("WITH_IOS_DEVICE_DETECTION=0");
			}
		}
	}
}
