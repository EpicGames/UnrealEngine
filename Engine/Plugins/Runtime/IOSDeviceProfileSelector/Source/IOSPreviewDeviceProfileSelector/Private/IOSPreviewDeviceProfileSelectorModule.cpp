// Copyright Epic Games, Inc. All Rights Reserved.

#include "IOSPreviewDeviceProfileSelectorModule.h"
#include "HAL/PlatformMisc.h"
#include "Modules/ModuleManager.h"
#include "PIEPreviewDeviceSpecification.h"
#include "JsonObjectConverter.h"
#include "Dom/JsonObject.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Regex.h"
#include "Misc/ConfigCacheIni.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Interfaces/IPluginManager.h"
#if WITH_IOS_DEVICE_DETECTION
#include "IOSDeviceHelper.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogIOSDPSelector, Log, All)
#define IOS_DEFAULT_DEVICE_PROFILE_NAME TEXT("IOS")

IMPLEMENT_MODULE(FIOSPreviewDeviceProfileSelectorModule, IOSPreviewDeviceProfileSelector);

struct FIOSPreviewDeviceSpec
{
	float ResX;
	float ResY;
};

static TMap<TArray<FString>, FIOSPreviewDeviceSpec> DeviceModelToDeviceSpec;

void FIOSPreviewDeviceProfileSelectorModule::StartupModule()
{
	FString IniPath = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("IOSDeviceProfileSelector"))->GetBaseDir(), TEXT("Config/DeviceModelsInfo.ini"));
	TArray<FString> RawInputs;
	FConfigFile DeviceModelsInfo;
	DeviceModelsInfo.Read(IniPath);
	if (DeviceModelsInfo.GetArray(TEXT("DeviceModelToDeviceSpec"), TEXT("Entry"), RawInputs))
	{
		for (const FString& Line : RawInputs)
		{
			FString RawDeviceModels, RawResXValues, RawResYValues;
			if (FParse::Value(*Line, TEXT("Devices="), RawDeviceModels) &&
				FParse::Value(*Line, TEXT("ResX="), RawResXValues) &&
				FParse::Value(*Line, TEXT("ResY="), RawResYValues))
			{
				RawDeviceModels = RawDeviceModels.TrimQuotes().TrimStartAndEnd();
				RawResXValues = RawResXValues.TrimQuotes().TrimStartAndEnd();
				RawResYValues = RawResYValues.TrimQuotes().TrimStartAndEnd();

				TArray<FString> DeviceModels;
				RawDeviceModels.ParseIntoArray(DeviceModels, TEXT("|"), true);

				FIOSPreviewDeviceSpec DeviceSpec;
				DeviceSpec.ResX = FCString::Atof(*RawResXValues);
				DeviceSpec.ResY = FCString::Atof(*RawResYValues);

				DeviceModelToDeviceSpec.Add(DeviceModels, DeviceSpec);
			}
		}
	}
}


void FIOSPreviewDeviceProfileSelectorModule::ShutdownModule()
{
}


FString const FIOSPreviewDeviceProfileSelectorModule::GetRuntimeDeviceProfileName()
{
	// We are not expecting this module to have GetRuntimeDeviceProfileName called directly.
	// IOS ProfileSelectorModule runtime is in FIOSDeviceProfileSelectorModule.
	// Use GetDeviceProfileName.
	checkNoEntry();
	return FString();
}

const FString GetDeviceProfileNameFromDeviceModel(const FString& DeviceModel)
{
	TArray<FString> MappingSections;
	MappingSections.Add(TEXT("IOSDeviceMappings"));
	MappingSections.Add(TEXT("IOSFallbackDeviceMappings"));
	for (FString& DeviceMappingSection : MappingSections)
	{
		TArray<FString> Mappings;
		FConfigCacheIni* PlatformEngineIni = FConfigCacheIni::ForPlatform("IOS");
		if (PlatformEngineIni && PlatformEngineIni->GetSection(*DeviceMappingSection, Mappings, GDeviceProfilesIni))
		{
			for (const FString& MappingString : Mappings)
			{
				FString MappingRegex, ProfileName;
				if (MappingString.Split(TEXT("="), &MappingRegex, &ProfileName))
				{
					const FRegexPattern RegexPattern(MappingRegex);
					FRegexMatcher RegexMatcher(RegexPattern, *DeviceModel);
					if (RegexMatcher.FindNext())
					{
						return ProfileName;
					}
				}
				else
				{
					FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Invalid %s: %s") LINE_TERMINATOR, *DeviceMappingSection, *MappingString);
				}
			}
		}
	}

	return IOS_DEFAULT_DEVICE_PROFILE_NAME;
}

const FString FIOSPreviewDeviceProfileSelectorModule::GetDeviceProfileName()
{
	// ensure SelectorProperties does actually contain our parameters
	FString IOSDeviceProfileName = IOS_DEFAULT_DEVICE_PROFILE_NAME;
	check(SelectorProperties.Num() > 0);
	
	FString DeviceModel;
	if (GetSelectorPropertyValue(TEXT("SRC_DeviceModel"), DeviceModel))
	{
		IOSDeviceProfileName = GetDeviceProfileNameFromDeviceModel(DeviceModel);
	}
	return IOSDeviceProfileName;
}

#if WITH_EDITOR

bool FIOSPreviewDeviceProfileSelectorModule::CanExportDeviceParametersToJson()
{ 
#if WITH_IOS_DEVICE_DETECTION
	return true; 
#else
	return false;
#endif
};

static bool GetDeviceSpecsFromJson(const FString& JsonLocation, FPIEPreviewDeviceSpecifications& OutDeviceSpecs)
{
	static TMap<const FString, FPIEPreviewDeviceSpecifications> FileLocationToDeviceSpec;
	if (FPIEPreviewDeviceSpecifications* CachedDeviceSpecs = FileLocationToDeviceSpec.Find(JsonLocation))
	{
		OutDeviceSpecs = *CachedDeviceSpecs;
		return true;
	}

	TSharedPtr<FJsonObject> JsonRootObject;
	FString Json;

	if (FFileHelper::LoadFileToString(Json, *JsonLocation))
	{
		TSharedRef<TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(Json);
		FJsonSerializer::Deserialize(JsonReader, JsonRootObject);
	}

	if (JsonRootObject.IsValid() && FJsonObjectConverter::JsonAttributesToUStruct(JsonRootObject->Values, FPIEPreviewDeviceSpecifications::StaticStruct(), &OutDeviceSpecs, 0, 0))
	{
		if (OutDeviceSpecs.IOSProperties.Version == IOS_DEVICE_PROPERTIES_VERSION)
		{
			FileLocationToDeviceSpec.Add(JsonLocation, OutDeviceSpecs);
		}
		return true;
	}

	return false;
}

bool FIOSPreviewDeviceProfileSelectorModule::IsJsonCompatible(const FString& JsonLocation)
{
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	if (GetDeviceSpecsFromJson(JsonLocation, DeviceSpecs))
	{
		if (DeviceSpecs.IOSProperties.Version > IOS_DEVICE_PROPERTIES_VERSION)
		{
			UE_LOGF(LogIOSDPSelector, Log, "Json %ls was saved with a newer version than this Engine Version supports(Found v%d, Expected v%d), please update your Json by connecting the same device that generated this Json and call Generate Preview Json.",
			*JsonLocation, DeviceSpecs.IOSProperties.Version, IOS_DEVICE_PROPERTIES_VERSION);
			return false;
		}
		else if (DeviceSpecs.DevicePlatform != EPIEPreviewDeviceType::IOS)
		{
			UE_LOGF(LogIOSDPSelector, Log, "Json %ls was saved with a different DevicePlatform than IOS. ", *JsonLocation);
			return false;
		}
	}
	else
	{
		return false;
	}

	return true;
}

#if WITH_IOS_DEVICE_DETECTION
void ExportDeviceProfile(const FString& OutPath, const FString& DeviceName)
{
	// instantiate an FPIEPreviewDeviceSpecifications instance and its values
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	DeviceSpecs.DevicePlatform = EPIEPreviewDeviceType::IOS;
	DeviceSpecs.IOSProperties.Version = IOS_DEVICE_PROPERTIES_VERSION;
	DeviceSpecs.IOSProperties.DeviceModel = DeviceName;
	// create a JSon object from the above structure
	TSharedPtr<FJsonObject> JsonObject = FJsonObjectConverter::UStructToJsonObject<FPIEPreviewDeviceSpecifications>(DeviceSpecs);

	// remove Android and switch fields
	JsonObject->RemoveField(TEXT("AndroidProperties"));
	JsonObject->RemoveField(TEXT("switchProperties"));

	// serialize the JSon object to string
	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);

	// export file to disk
	FFileHelper::SaveStringToFile(OutputString, *OutPath);
}
#endif

void FIOSPreviewDeviceProfileSelectorModule::ExportDeviceParametersToJson(FString& FolderLocation, TArray<FString>& OutExportedFiles)
{
#if WITH_IOS_DEVICE_DETECTION
	TSet<FString> AlreadyExported;
	TArray<FLibIMobileDevice> ParsedDevices = FIOSDeviceHelper::GetLibIMobileDevices();
	for (FLibIMobileDevice& Device : ParsedDevices)
	{
		FString DeviceModel = Device.DeviceType;
		if (!AlreadyExported.Find(DeviceModel))
		{
			FString ExportPath = FolderLocation / (GetDeviceProfileNameFromDeviceModel(DeviceModel) + TEXT(".json"));
			OutExportedFiles.Add(ExportPath);
			ExportDeviceProfile(ExportPath, DeviceModel);
			AlreadyExported.Add(DeviceModel);
		}
	}
#endif
}

static void MakeOutdatedDeviceSpecCompatible(FPIEPreviewDeviceSpecifications& OutDeviceSpecs)
{

}

void FIOSPreviewDeviceProfileSelectorModule::GetDeviceParametersFromJson(FString& JsonLocation, TMap<FString, FString>& OutDeviceParameters)
{
	FPIEPreviewDeviceSpecifications DeviceSpecs;
	if (GetDeviceSpecsFromJson(JsonLocation, DeviceSpecs))
	{
		if(DeviceSpecs.IOSProperties.Version < IOS_DEVICE_PROPERTIES_VERSION)
		{
			MakeOutdatedDeviceSpecCompatible(DeviceSpecs);
			UE_LOGF(LogIOSDPSelector, Log, "Json %ls is using an Outdated Json Version(Found v%d, Expected v%d), please update your Json by connecting the same device that generated this Json and call Generate Preview Json.", 
			*JsonLocation, DeviceSpecs.IOSProperties.Version, IOS_DEVICE_PROPERTIES_VERSION);
		}

		FPIEIOSDeviceProperties& IOSProperties = DeviceSpecs.IOSProperties;
		OutDeviceParameters.Add((TEXT("SRC_DeviceModel")), IOSProperties.DeviceModel);
	}
	else
	{
		UE_LOGF(LogIOSDPSelector, Log, "Json %ls failed to load", *JsonLocation);
		return;
	}
}

const EShaderPlatform FIOSPreviewDeviceProfileSelectorModule::GetPreviewShaderPlatform()
{
	EShaderPlatform ParentShaderPlatform = SP_METAL_ES3_1_IOS;

	EShaderPlatform PreviewPlatform = SP_NumPlatforms;
	for (int32 PlatformIndex = SP_CUSTOM_PLATFORM_FIRST; PlatformIndex < SP_NumPlatforms; PlatformIndex++)
	{
		EShaderPlatform PreviewPlatformToCheck = EShaderPlatform(PlatformIndex);
		if (FDataDrivenShaderPlatformInfo::IsValid(PreviewPlatformToCheck) &&
			FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(PreviewPlatformToCheck) &&
			FDataDrivenShaderPlatformInfo::GetPreviewShaderPlatformParent(PreviewPlatformToCheck) == ParentShaderPlatform)
		{
			PreviewPlatform = PreviewPlatformToCheck;
			break;
		}
	}
	return PreviewPlatform;
}

void FIOSPreviewDeviceProfileSelectorModule::SetSelectorProperties(const TMap<FString, FString>& InSelectorProperties)
{
	const FString IPhone = TEXT("iPhone");
	const FString IPad = TEXT("iPad");
	if (const FString* DeviceModel = InSelectorProperties.Find(TEXT("SRC_DeviceModel")))
	{
		// Fallbacks if we do not find the model.
		if (DeviceModel->StartsWith(IPhone))
		{
			ConstrainedAspectRatio = 19.5f/9.f;
		}
		else if (DeviceModel->StartsWith(IPad))
		{
			ConstrainedAspectRatio = 4.f/3.f;
		}

		FIOSPreviewDeviceSpec DeviceSpec;
		for (const auto& Pair : DeviceModelToDeviceSpec)
		{
			for (const FString& DeviceModelName : Pair.Key)
			{
				FString MajorModelName;
				FString MinorModelNumberArray;
				DeviceModelName.Split(":", &MajorModelName, &MinorModelNumberArray);

				TArray<FString> MinorModelNumbers;
				MinorModelNumberArray.ParseIntoArray(MinorModelNumbers, TEXT(","), true);
				for (const FString& MinorModelNumber : MinorModelNumbers)
				{
					FString FullModelNumberToVerify = MajorModelName + TEXT(",") + MinorModelNumber;
					if (*DeviceModel == FullModelNumberToVerify)
					{
						DeviceSpec = Pair.Value;
						if(DeviceSpec.ResY != 0.f)
						{
							ConstrainedAspectRatio = DeviceSpec.ResX / DeviceSpec.ResY;
						}
						break;
					}
				}
			}
		}
	}
	SelectorProperties = InSelectorProperties;
}

bool FIOSPreviewDeviceProfileSelectorModule::GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT)
{
	if (const FString* Found = SelectorProperties.Find(PropertyType.ToString()))
	{
		PropertyValueOUT = *Found;
		return true;
	}

	return false;
}

#endif