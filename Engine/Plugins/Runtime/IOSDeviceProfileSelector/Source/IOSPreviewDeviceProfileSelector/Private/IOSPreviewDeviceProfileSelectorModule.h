// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDeviceProfileSelectorModule.h"

/**
 * Implements the IOS Device Profile Selector module.
 */
class FIOSPreviewDeviceProfileSelectorModule
	: public IDeviceProfileSelectorModule
{
public:

	//~ Begin IDeviceProfileSelectorModule Interface
	virtual const FString GetDeviceProfileName() override;
	virtual const FString GetRuntimeDeviceProfileName() override;
#if WITH_EDITOR
	virtual bool CanExportDeviceParametersToJson() override;
	virtual bool IsJsonCompatible(const FString& JsonLocation) override;
	virtual void ExportDeviceParametersToJson(FString& FolderLocation, TArray<FString>& OutExportedFiles) override;
	virtual void GetDeviceParametersFromJson(FString& JsonLocation, TMap<FString, FString>& OutDeviceParameters) override;
	virtual bool CanGetDeviceParametersFromJson() override { return true; }
	virtual const EShaderPlatform GetPreviewShaderPlatform() override;
	virtual void SetSelectorProperties(const TMap<FString, FString>& InSelectorProperties) override;
	virtual bool GetSelectorPropertyValue(const FName& PropertyType, FString& PropertyValueOUT) override;
#endif
	//~ End IDeviceProfileSelectorModule Interface


	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface

	
	/**
	 * Virtual destructor.
	 */
	virtual ~FIOSPreviewDeviceProfileSelectorModule()
	{
	}
private:
	TMap<FString, FString> SelectorProperties;
};
